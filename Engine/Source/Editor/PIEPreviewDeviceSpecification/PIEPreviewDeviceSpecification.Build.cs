// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class PIEPreviewDeviceSpecification : ModuleRules
	{
        public PIEPreviewDeviceSpecification(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
				}
				);
		}
	}
}
