// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewDeviceSpecification.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PIEPreviewDeviceSpecification)

class FPIEPreviewDeviceSpecificationModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FPIEPreviewDeviceSpecificationModule, PIEPreviewDeviceSpecification);

UPIEPreviewDeviceSpecification::UPIEPreviewDeviceSpecification(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
