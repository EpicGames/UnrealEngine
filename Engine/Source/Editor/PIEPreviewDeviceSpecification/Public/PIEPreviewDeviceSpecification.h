// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

#include "PIEPreviewDeviceSpecification.generated.h"

// If you modify the logic or structure of the FPIEPreviewDeviceSpecifications
// Please bump the version for the platform affected and add backwards compatibility to the MakeOutdatedDeviceSpecCompatible function of the platform's Preview Device Profile Selector.
#define IOS_DEVICE_PROPERTIES_VERSION 1
#define ANDROID_DEVICE_PROPERTIES_VERSION 1

UENUM()
enum class EPIEPreviewDeviceType : uint8
{
	Unset,
	Android,
	IOS,
	TVOS,
	Switch,
	MAX,
};

UCLASS(MinimalAPI)
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") UPIEPreviewDeviceSpecification : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY()
	EPIEPreviewDeviceType PreviewDeviceType;

	UPROPERTY()
	FString GPUFamily;
	UPROPERTY()
	FString GLVersion;
	UPROPERTY()
	FString VulkanVersion;
	UPROPERTY()
	FString AndroidVersion;
	UPROPERTY()
	FString DeviceMake;
	UPROPERTY()
	FString DeviceModel;
	UPROPERTY()
	FString DeviceBuildNumber;
	UPROPERTY()
	bool UsingHoudini;
	UPROPERTY()
	FString Hardware;
	UPROPERTY()
	FString Chipset;
};

USTRUCT()
struct UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIERHIOverrideState
{
public:
	GENERATED_USTRUCT_BODY()
	UPROPERTY()
	int32 MaxShadowDepthBufferSizeX = 0;
	UPROPERTY()
	int32 MaxShadowDepthBufferSizeY = 0;
	UPROPERTY()
	int32 MaxTextureDimensions = 0;
	UPROPERTY()
	int32 MaxCubeTextureDimensions = 0;
	UPROPERTY()
	bool SupportsRenderTargetFormat_PF_G8 = false;
	UPROPERTY()
	bool SupportsRenderTargetFormat_PF_FloatRGBA = false;
	UPROPERTY()
	bool SupportsMultipleRenderTargets = false;
};

USTRUCT()
struct FPIEAndroidDeviceProperties
{
public:
	GENERATED_USTRUCT_BODY()
	UPROPERTY()
	int32 Version = 0;
	UPROPERTY()
	FString GPUFamily;
	UPROPERTY()
	FString GLVersion;
	UPROPERTY()
	FString VulkanVersion;
	UPROPERTY()
	FString AndroidVersion;
	UPROPERTY()
	FString DeviceMake;
	UPROPERTY()
	FString DeviceModel;
	UPROPERTY()
	FString DeviceBuildNumber;
	UPROPERTY()
	bool VulkanAvailable = false;
	UPROPERTY()
	bool UsingHoudini = false;
	UPROPERTY()
	FString Hardware;
	UPROPERTY()
	FString Chipset;
	UPROPERTY()
	FString TotalPhysicalGB;
	UPROPERTY()
	FString HMDSystemName;

	UPROPERTY()
	bool SM5Available = false;
};

USTRUCT()
struct FPIEIOSDeviceProperties
{
public:
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	int32 Version = 0;

	UPROPERTY()
	FString DeviceModel;

	UPROPERTY()
	float NativeScaleFactor = 0.0f;
};

USTRUCT()
struct FPIESwitchDeviceProperties
{
public:
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	bool Docked = false;
};


USTRUCT()
struct UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewDeviceBezelViewportRect
{
public:
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	int32 X = 0;
	UPROPERTY()
	int32 Y = 0;
	UPROPERTY()
	int32 Width = 0;
	UPROPERTY()
	int32 Height = 0;
}; 

USTRUCT()
struct UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEBezelProperties
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString DeviceBezelFile;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta = (DeprecatedProperty))
	FPIEPreviewDeviceBezelViewportRect BezelViewportRect;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

USTRUCT()
struct FPIEPreviewDeviceSpecifications
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	EPIEPreviewDeviceType DevicePlatform = EPIEPreviewDeviceType::Unset;
	UPROPERTY()
	int32 ResolutionX = 0;
	UPROPERTY()
	int32 ResolutionY = 0;
	UPROPERTY()
	int32 ResolutionYImmersiveMode = 0;

	UPROPERTY()
	float InsetsLeft = 0.0f;
	
	UPROPERTY()
	float InsetsTop = 0.0f;
	
	UPROPERTY()
	float InsetsRight = 0.0f;
	
	UPROPERTY()
	float InsetsBottom = 0.0f;

	UPROPERTY()
	int32 PPI = 0;

	UPROPERTY()
	TArray<float> ScaleFactors;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta = (DeprecatedProperty))
	FPIEBezelProperties BezelProperties;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	FPIEAndroidDeviceProperties AndroidProperties;

	UPROPERTY()
	FPIEIOSDeviceProperties IOSProperties;

	UPROPERTY()
	FPIESwitchDeviceProperties SwitchProperties;
};
